{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Sample Mesh Reconstructor",
	"Description" : "Sample of how to drive mesh reconstruction. Generates dummy geometry to demonstrate API usage.",
	"Category" : "Virtual Reality",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "DummyMeshReconstructor",
			"Type" : "Runtime",
			"LoadingPhase" : "PostConfigInit",
			"PlatformAllowList" :
			[
				"Android",
				"Win64"
			]
		}
	]
}
