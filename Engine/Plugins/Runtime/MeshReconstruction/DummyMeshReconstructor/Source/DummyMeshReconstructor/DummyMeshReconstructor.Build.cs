// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DummyMeshReconstructor : ModuleRules
	{
		public DummyMeshReconstructor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"MRMesh",
				}
			);
		}
	}
}
