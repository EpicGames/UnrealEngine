// Copyright Epic Games, Inc. All Rights Reserved.

#include "DummyMeshReconstructor.h"

#include "HAL/Runnable.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeBool.h"
#include "MRMeshComponent.h"
#include "DynamicMeshBuilder.h"
#include "Containers/Queue.h"

static const FVector BRICK_SIZE(256.0f, 256.0f, 256.0f);
static const int32 BRICK_COORD_RANDMAX = 8;

class FDummyMeshReconstructor : public FRunnable
{
public:
	FDummyMeshReconstructor()
	: bKeepRunning(false)
	, bResendAllData(false)
	, TargetMRMesh(nullptr )
	, ReconstructorThread(nullptr)
	, LastClearTime(FApp::GetCurrentTime())
	{
	
		// Pre-allocate the reconstructed geometry data.
		// Re-allocating this array will cause dangling pointers from MRMeshComponent.
		ReconstructedGeometry.Reserve(BRICK_COORD_RANDMAX*BRICK_COORD_RANDMAX*BRICK_COORD_RANDMAX);
	}

	~FDummyMeshReconstructor()
	{
		// Stop the geometry generator thread.
		if (ReconstructorThread.IsValid())
		{
			ReconstructorThread->Kill(true);
		}
	}

	/**
	 * Get the MRMeshComponent that is currently presenting our data.
	 * Used for checking against re-connects to the same component.
	 */
	const IMRMesh* GetTargetMRMesh() const { return TargetMRMesh; }

	void SetTargetMRMesh(IMRMesh* InTaget) { TargetMRMesh = InTaget; }

	void StopThread()
	{
		// Stop the geometry generator thread.
		if (ReconstructorThread.IsValid())
		{
			ReconstructorThread->Kill(true);
		}
	}

	void StartThread()
	{
		// Start a new thread.
		bKeepRunning.AtomicSet(true);
		ReconstructorThread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("Dummy Mesh Reconstructor")));
	}

	/** Re-send all the geometry data to the paired MRMeshComponent */
	void ResendAllData()
	{
		bResendAllData.AtomicSet(true);
	}

	bool IsRunning() const { return bKeepRunning; }

private:
	//~ FRunnable
	virtual uint32 Run() override
	{
		//
		// Main geometry generator loop
		//

		while (bKeepRunning)
		{
			if (TargetMRMesh != nullptr)
			{
				if (bResendAllData)
				{
					// The component requested that we re-send all the data.
					for (FPayload& Payload : ReconstructedGeometry)
					{
						TargetMRMesh->SendBrickData(IMRMesh::FSendBrickDataArgs
						{
							nullptr,
							Payload.BrickId,
							Payload.PositionData,
							Payload.UVData,
							Payload.Tangents,
							Payload.ColorData,
							Payload.Indices
						});
					}
					bResendAllData.AtomicSet(false);
				}

				// Send newly-generated brick.
				{
					// Allocate a new brick. We own this memory.
					const int32 NewPayloadIndex = NewRandomPayload();
					TargetMRMesh->SendBrickData(IMRMesh::FSendBrickDataArgs
					{
						nullptr,
						ReconstructedGeometry[NewPayloadIndex].BrickId,
						ReconstructedGeometry[NewPayloadIndex].PositionData,
						ReconstructedGeometry[NewPayloadIndex].UVData,
						ReconstructedGeometry[NewPayloadIndex].Tangents,
						ReconstructedGeometry[NewPayloadIndex].ColorData,
						ReconstructedGeometry[NewPayloadIndex].Indices
					});
				}

				const double CurrentTime = FApp::GetCurrentTime();
				if (CurrentTime - LastClearTime > 10.0f)
				{
					LastClearTime = CurrentTime;
					TargetMRMesh->ClearAllBrickData();
				}
			}

			FPlatformProcess::Sleep(1.0f/5.0f);
		}

		return 0;
	}

	virtual void Stop() override
	{
		bKeepRunning.AtomicSet(false);
	}
	//~ FRunnable

	struct FPayload
	{
		IMRMesh::FBrickId BrickId;
		FIntVector BrickCoords;
		TArray<FVector3f> PositionData;
		TArray<FVector2D> UVData;
		TArray<FPackedNormal> Tangents;
		TArray<FColor> ColorData;
		TArray<uint32> Indices;
	};
	
	int32 NewRandomPayload()
	{
		static const int32 MIN_BOXES = 0;
		static const int32 MAX_BOXES = 20;
		static const FBox RANDOM_SIZE_BOX = FBox(FVector::ZeroVector, 0.25f*BRICK_SIZE);

		const int32 NumBoxes = FMath::FloorToInt((float)FMath::RandRange(MIN_BOXES, MAX_BOXES));
		const int32 NumUniqueVerts = NumBoxes * 8;
		const int32 NumTris = NumBoxes * 6 * 2; // 2 tris per box face
		const int32 NumVertIndices = 3 * NumTris;

		static IMRMesh::FBrickId NextBrickId = 0;
		const IMRMesh::FBrickId BrickId = ++NextBrickId;
		
		const FIntVector BrickCoords(
			FMath::FloorToInt((float)FMath::RandRange(0, BRICK_COORD_RANDMAX)),
			FMath::FloorToInt((float)FMath::RandRange(0, BRICK_COORD_RANDMAX)),
			FMath::FloorToInt((float)FMath::RandRange(0, BRICK_COORD_RANDMAX))
		);

		const FVector BrickOrigin(BRICK_SIZE.X*BrickCoords.X, BRICK_SIZE.Y*BrickCoords.Y, BRICK_SIZE.Z*BrickCoords.Z);
		const FBox RandomLocationsBox = FBox(BrickOrigin, BrickOrigin + FVector(1024, 1024, 1024));

		//
		// !! Allocate a new Brick. We own this data.
		//
		ReconstructedGeometry.AddDefaulted(1);
		FPayload& NewPayload = ReconstructedGeometry.Last();
		NewPayload.BrickId = BrickId;
		NewPayload.BrickCoords = BrickCoords;
		NewPayload.PositionData.Reserve(NumUniqueVerts);
		NewPayload.UVData.Reserve(NumUniqueVerts);
		NewPayload.Tangents.Reserve(NumUniqueVerts);
		NewPayload.ColorData.Reserve(NumUniqueVerts);
		NewPayload.Indices.Reserve(NumVertIndices);	

		auto AddBox = [](FVector Origin, FVector Extents, FPayload& NewPayloadIn)
		{
			const int32 IndexOffset = NewPayloadIn.PositionData.Num();
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X + Extents.X, Origin.Y - Extents.Y, Origin.Z + Extents.Z));  // IndexOffset+0
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X + Extents.X, Origin.Y + Extents.Y, Origin.Z + Extents.Z));  // IndexOffset+1
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X + Extents.X, Origin.Y + Extents.Y, Origin.Z - Extents.Z));  // IndexOffset+2
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X + Extents.X, Origin.Y - Extents.Y, Origin.Z - Extents.Z));  // IndexOffset+3
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X - Extents.X, Origin.Y - Extents.Y, Origin.Z + Extents.Z));  // IndexOffset+4
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X - Extents.X, Origin.Y + Extents.Y, Origin.Z + Extents.Z));  // IndexOffset+5
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X - Extents.X, Origin.Y + Extents.Y, Origin.Z - Extents.Z));  // IndexOffset+6
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());
			NewPayloadIn.PositionData.Emplace(FVector(Origin.X - Extents.X, Origin.Y - Extents.Y, Origin.Z - Extents.Z));  // IndexOffset+7
			NewPayloadIn.ColorData.Emplace(FColor::MakeRandomColor());

			NewPayloadIn.UVData.Emplace(FVector2D(0, 0));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(0, 1));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(1,0));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(1, 1));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(0,0));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(0,1));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(1, 0));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));

			NewPayloadIn.UVData.Emplace(FVector2D(1,1));
			NewPayloadIn.Tangents.Emplace(FPackedNormal(FVector4(1, 0, 0, 1)));
			//NewPayloadIn.TangentZData.Emplace(FPackedNormal(0, 0, 1, 1));


			NewPayloadIn.Indices.Add(IndexOffset + 0);
			NewPayloadIn.Indices.Add(IndexOffset + 1);
			NewPayloadIn.Indices.Add(IndexOffset + 2);

			NewPayloadIn.Indices.Add(IndexOffset + 0);
			NewPayloadIn.Indices.Add(IndexOffset + 2);
			NewPayloadIn.Indices.Add(IndexOffset + 3);

			NewPayloadIn.Indices.Add(IndexOffset + 0);
			NewPayloadIn.Indices.Add(IndexOffset + 4);
			NewPayloadIn.Indices.Add(IndexOffset + 1);

			NewPayloadIn.Indices.Add(IndexOffset + 1);
			NewPayloadIn.Indices.Add(IndexOffset + 4);
			NewPayloadIn.Indices.Add(IndexOffset + 5);

			NewPayloadIn.Indices.Add(IndexOffset + 7);
			NewPayloadIn.Indices.Add(IndexOffset + 5);
			NewPayloadIn.Indices.Add(IndexOffset + 4);

			NewPayloadIn.Indices.Add(IndexOffset + 6);
			NewPayloadIn.Indices.Add(IndexOffset + 5);
			NewPayloadIn.Indices.Add(IndexOffset + 7);

			NewPayloadIn.Indices.Add(IndexOffset + 7);
			NewPayloadIn.Indices.Add(IndexOffset + 3);
			NewPayloadIn.Indices.Add(IndexOffset + 2);

			NewPayloadIn.Indices.Add(IndexOffset + 7);
			NewPayloadIn.Indices.Add(IndexOffset + 2);
			NewPayloadIn.Indices.Add(IndexOffset + 6);

			NewPayloadIn.Indices.Add(IndexOffset + 7);
			NewPayloadIn.Indices.Add(IndexOffset + 4);
			NewPayloadIn.Indices.Add(IndexOffset + 0);

			NewPayloadIn.Indices.Add(IndexOffset + 7);
			NewPayloadIn.Indices.Add(IndexOffset + 0);
			NewPayloadIn.Indices.Add(IndexOffset + 3);

			NewPayloadIn.Indices.Add(IndexOffset + 1);
			NewPayloadIn.Indices.Add(IndexOffset + 5);
			NewPayloadIn.Indices.Add(IndexOffset + 6);

			NewPayloadIn.Indices.Add(IndexOffset + 2);
			NewPayloadIn.Indices.Add(IndexOffset + 1);
			NewPayloadIn.Indices.Add(IndexOffset + 6);
		};


		for (int i = 0; i < NumBoxes; ++i)
		{
			auto wat = FVector(FMath::RandRange(5.0f, 100.0f), FMath::RandRange(5.0f, 100.0f), FMath::RandRange(5.0f, 100.0f));

			AddBox(FMath::RandPointInBox(RandomLocationsBox), FMath::RandPointInBox(RANDOM_SIZE_BOX), NewPayload);
		}

		return ReconstructedGeometry.Num() - 1;
	}

	
	FThreadSafeBool bKeepRunning;
	FThreadSafeBool bResendAllData;
	IMRMesh* TargetMRMesh;	
	TArray<FPayload> ReconstructedGeometry;
	TUniquePtr<FRunnableThread> ReconstructorThread;
	double LastClearTime = 0;
};



void UDummyMeshReconstructor::StartReconstruction()
{
	// Implicitly starts the reconstructor
	EnsureImplExists();
	if (!ReconstructorImpl->IsRunning())
	{
		ReconstructorImpl->StartThread();
	}
}

void UDummyMeshReconstructor::StopReconstruction()
{
	EnsureImplExists();
	ReconstructorImpl->StopThread();
}

void UDummyMeshReconstructor::PauseReconstruction()
{
	EnsureImplExists();
	ReconstructorImpl->StopThread();
}

bool UDummyMeshReconstructor::IsReconstructionStarted() const
{
	return ReconstructorImpl.IsValid();
}

bool UDummyMeshReconstructor::IsReconstructionPaused() const
{
	return ReconstructorImpl.IsValid() && !ReconstructorImpl->IsRunning();
}

void UDummyMeshReconstructor::ConnectMRMesh(UMRMeshComponent* Mesh)
{
	EnsureImplExists();
	ReconstructorImpl->SetTargetMRMesh(Mesh);
}

void UDummyMeshReconstructor::DisconnectMRMesh()
{
	if (ReconstructorImpl.IsValid())
	{
		ReconstructorImpl.Reset();
	}
}

void UDummyMeshReconstructor::EnsureImplExists()
{
	if (!ReconstructorImpl.IsValid())
	{
		ReconstructorImpl = MakeShareable(new FDummyMeshReconstructor());
	}
}

