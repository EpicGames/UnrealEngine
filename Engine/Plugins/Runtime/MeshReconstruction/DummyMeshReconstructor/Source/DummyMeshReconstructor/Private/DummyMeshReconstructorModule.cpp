// Copyright Epic Games, Inc. All Rights Reserved.

#include "DummyMeshReconstructorModule.h"
#include "BaseMeshReconstructorModule.h"
#include "Modules/ModuleManager.h"
#include "HAL/Runnable.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "HAL/ThreadSafeBool.h"
#include "MRMeshComponent.h"
#include "DynamicMeshBuilder.h"
#include "Containers/Queue.h"

// Thin wrapper around the running thread that does all the reconstruction.
class FDummyMeshReconstructorModule : public FBaseMeshReconstructorModule
{
};

IMPLEMENT_MODULE(FDummyMeshReconstructorModule, DummyMeshReconstructor);
