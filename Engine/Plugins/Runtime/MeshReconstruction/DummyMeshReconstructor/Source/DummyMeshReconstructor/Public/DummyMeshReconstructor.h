// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshReconstructorBase.h"

#include "DummyMeshReconstructor.generated.h"

class FDummyMeshReconstructor;

UCLASS(BlueprintType, meta = (Experimental))
class DUMMYMESHRECONSTRUCTOR_API UDummyMeshReconstructor : public UMeshReconstructorBase
{
	GENERATED_BODY()

public:
	//~ UMeshReconstructorBase
	virtual void StartReconstruction() override;
	
	virtual void StopReconstruction() override;
	
	virtual void PauseReconstruction() override;
	
	virtual bool IsReconstructionStarted() const override;
	
	virtual bool IsReconstructionPaused() const override;

	virtual void ConnectMRMesh(UMRMeshComponent* Mesh) override;

	virtual void DisconnectMRMesh() override;
	//~ UMeshReconstructorBase

private:
	void EnsureImplExists();
	TSharedPtr<FDummyMeshReconstructor> ReconstructorImpl;
};